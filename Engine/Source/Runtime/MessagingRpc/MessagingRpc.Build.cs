// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MessagingRpc : ModuleRules
	{
		public MessagingRpc(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					"Messaging",
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"MessagingCommon",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Messaging",
				});
		}
	}
}
