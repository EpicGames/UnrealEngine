// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageRpcClient.h"

#include "Containers/ArrayBuilder.h"
#include "Containers/Ticker.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"

#include "IMessageRpcCall.h"
#include "MessageRpcDefines.h"
#include "MessageRpcMessages.h"
#include "RpcMessage.h"


/* FMessageRpcClient structors
 *****************************************************************************/

FMessageRpcClient::FMessageRpcClient()
	: FMessageRpcClient(FMessageEndpoint::Builder(TEXT("FMessageRpcClient")))
{ 
}

FMessageRpcClient::FMessageRpcClient(const FString& InDebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: FMessageRpcClient(FMessageEndpoint::Builder(*InDebugName, InMessageBus))
{
}

FMessageRpcClient::FMessageRpcClient(FMessageEndpointBuilder&& InEndpointBuilder)
{
	MessageEndpoint = InEndpointBuilder
		.Handling<FMessageRpcProgress>(this, &FMessageRpcClient::HandleProgressMessage)
		.WithCatchall(this, &FMessageRpcClient::HandleRpcMessages);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FMessageRpcClient::HandleTicker), MESSAGE_RPC_RETRY_INTERVAL);
}

FMessageRpcClient::~FMessageRpcClient()
{
	FTSTicker::RemoveTicker(TickerHandle);
}


/* IMessageRpcClient interface
 *****************************************************************************/

void FMessageRpcClient::Connect(const FMessageAddress& InServerAddress)
{
	Disconnect();
	ServerAddress = InServerAddress;
}


void FMessageRpcClient::Disconnect()
{
	for (TMap<FGuid, TSharedPtr<IMessageRpcCall>>::TIterator It(Calls); It; ++It)
	{
		It.Value()->TimeOut();
	}

	Calls.Empty();
	ServerAddress.Invalidate();
}


bool FMessageRpcClient::IsConnected() const
{
	return ServerAddress.IsValid();
}


/* FMessageRpcClient implementation
 *****************************************************************************/

TSharedPtr<IMessageRpcCall> FMessageRpcClient::FindCall(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	auto Request = static_cast<const FRpcMessage*>(Context->GetMessage());
	return Calls.FindRef(Request->CallId);
}


void FMessageRpcClient::SendCall(const TSharedPtr<IMessageRpcCall>& Call)
{
	if (ServerAddress.IsValid())
	{
		MessageEndpoint->Send(
			Call->ConstructMessage(),
			Call->GetMessageType(),
			EMessageFlags::None,
			nullptr,
			TArrayBuilder<FMessageAddress>().Add(ServerAddress),
			FTimespan::Zero(),
			FDateTime::MaxValue()
		);
	}
}


/* IMessageRpcClient interface
 *****************************************************************************/

void FMessageRpcClient::AddCall(const TSharedRef<IMessageRpcCall>& Call)
{
	Calls.Add(Call->GetId(), Call);
	SendCall(Call);
}


void FMessageRpcClient::CancelCall(const FGuid& CallId)
{
	TSharedPtr<IMessageRpcCall> Call;
	
	if (Calls.RemoveAndCopyValue(CallId, Call))
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FMessageRpcCancel>(CallId), ServerAddress);
	}
}


/* FMessageRpcClient event handlers
 *****************************************************************************/

void FMessageRpcClient::HandleProgressMessage(const FMessageRpcProgress& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	TSharedPtr<IMessageRpcCall> Call = Calls.FindRef(Message.CallId);

	if (Call.IsValid())
	{
		Call->UpdateProgress(Message.Completion, FText::FromString(Message.StatusText));
	}
}


void FMessageRpcClient::HandleRpcMessages(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const TWeakObjectPtr<UScriptStruct>& MessageTypeInfo = Context->GetMessageTypeInfo();

	if (!MessageTypeInfo.IsValid())
	{
		return;
	}

	if (MessageTypeInfo->IsChildOf(FRpcMessage::StaticStruct()))
	{
		auto Request = static_cast<const FRpcMessage*>(Context->GetMessage());
		TSharedPtr<IMessageRpcCall> Call;

		if (Calls.RemoveAndCopyValue(Request->CallId, Call))
		{
			Call->Complete(Context);
		}
	}
}


bool FMessageRpcClient::HandleTicker(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageRpcClient_HandleTicker);

	const FDateTime UtcNow = FDateTime::UtcNow();

	for (TMap<FGuid, TSharedPtr<IMessageRpcCall>>::TIterator It(Calls); It; ++It)
	{
		auto Call = It.Value();
		
		if ((UtcNow - Call->GetTimeCreated()) > FTimespan::FromSeconds(MESSAGE_RPC_RETRY_TIMEOUT))
		{
			It.RemoveCurrent();
			Call->TimeOut();
		}
		else if ((UtcNow - Call->GetLastUpdated()) > FTimespan::FromSeconds(MESSAGE_RPC_RETRY_INTERVAL))
		{
			SendCall(Call);
		}
	}

	return true;
}
