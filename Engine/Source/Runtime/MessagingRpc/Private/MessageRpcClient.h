// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "IMessageContext.h"
#include "IMessageRpcClient.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Containers/Ticker.h"

class FMessageEndpoint;
class IMessageBus;
class IMessageRpcCall;

struct FMessageRpcProgress;
struct FMessageEndpointBuilder;


/**
 * Implements an RPC client.
 */
class FMessageRpcClient
	: public IMessageRpcClient
{
public:

	/** Default constructor. */
	FMessageRpcClient();
	FMessageRpcClient(const FString& InDebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);
	
	/** Virtual destructor. */
	virtual ~FMessageRpcClient();

public:

	//~ IMessageRpcClient interface

	virtual void Connect(const FMessageAddress& InServerAddress) override;
	virtual void Disconnect() override;
	virtual bool IsConnected() const override;

protected:
	explicit FMessageRpcClient(FMessageEndpointBuilder&& InEndpointBuilder);

	/**
	 * Find the active RPC call for a received message.
	 *
	 * @param Context The context of the received message.
	 * @return The matching call object, or nullptr if not found.
	 */
	TSharedPtr<IMessageRpcCall> FindCall(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/**
	 * Sends an RPC call to the server.
	 *
	 * @param Call The RPC call to make.
	 */
	void SendCall(const TSharedPtr<IMessageRpcCall>& Call);

protected:

	//~ IMessageRpcClient interface

	virtual void AddCall(const TSharedRef<IMessageRpcCall>& Call) override;
	virtual void CancelCall(const FGuid& CallId) override;

private:

	/** Callback for FMessageRpcProgress messages. */
	void HandleProgressMessage(const FMessageRpcProgress& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for handling all message types. */
	void HandleRpcMessages(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for the ticker. */
	bool HandleTicker(float DeltaTime);

private:

	/** Active RPC calls. */
	TMap<FGuid, TSharedPtr<IMessageRpcCall>> Calls;

	/** Message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** The RPC server's address. */
	FMessageAddress ServerAddress;

	/** Handle to the registered ticker. */
	FTSTicker::FDelegateHandle TickerHandle;
};
