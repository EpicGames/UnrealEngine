// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** The interval at which progression are being sent to the client (in seconds). */
#ifndef MESSAGE_RPC_TICK_DELAY
#define MESSAGE_RPC_TICK_DELAY 0.1f
#endif

/** The interval at which calls are being re-sent to the server (in seconds). */
#ifndef MESSAGE_RPC_RETRY_INTERVAL
#define MESSAGE_RPC_RETRY_INTERVAL 1.0
#endif

/** The time after which calls time out (in seconds). */
#ifndef MESSAGE_RPC_RETRY_TIMEOUT
#define MESSAGE_RPC_RETRY_TIMEOUT 3.0
#endif

/** The interval at which progression are being sent to the client (in seconds). */
#ifndef MESSAGE_RPC_PROGRESS_INTERVAL
#define MESSAGE_RPC_PROGRESS_INTERVAL (0.25 * MESSAGE_RPC_RETRY_INTERVAL)
#endif