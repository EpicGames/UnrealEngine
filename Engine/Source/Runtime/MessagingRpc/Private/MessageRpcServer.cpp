// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageRpcServer.h"

#include "Async/IAsyncProgress.h"
#include "Containers/ArrayBuilder.h"
#include "Containers/Ticker.h"
#include "IMessageRpcHandler.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"

#include "RpcMessage.h"
#include "MessageRpcDefines.h"
#include "MessageRpcMessages.h"


/* FMessageRpcServer structors
 *****************************************************************************/

FMessageRpcServer::FMessageRpcServer()
	: FMessageRpcServer(FMessageEndpoint::Builder(TEXT("FMessageRpcServer")))
{
}

FMessageRpcServer::FMessageRpcServer(const FString& InDebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: FMessageRpcServer(FMessageEndpoint::Builder(*InDebugName, InMessageBus))
{
}

FMessageRpcServer::FMessageRpcServer(FMessageEndpointBuilder&& InEndpointBuilder)
{
	MessageEndpoint = InEndpointBuilder.WithCatchall(this, &FMessageRpcServer::HandleMessage);
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FMessageRpcServer::HandleTicker), MESSAGE_RPC_TICK_DELAY);
}

FMessageRpcServer::~FMessageRpcServer()
{
	FTSTicker::RemoveTicker(TickerHandle);
}


/* IMessageRpcServer interface
 *****************************************************************************/

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FMessageRpcServer::GetEndpoint() const
{
	return MessageEndpoint;
}

void FMessageRpcServer::AddHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IMessageRpcHandler>& Handler)
{
	Handlers.Add(RequestMessageType, Handler);
}


const FMessageAddress& FMessageRpcServer::GetAddress() const
{
	return MessageEndpoint->GetAddress();
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOnMessageRpcNoHandler& FMessageRpcServer::OnNoHandler()
{
	return NoHandlerDelegate;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FOnMessagePathNameRpcNoHandler& FMessageRpcServer::OnNoHandlerWithPathName()
{
	return NoHandlerDelegateWithPathName;
}

void FMessageRpcServer::SetSendProgressUpdate(bool InSendProgress)
{
	bSendProgress = InSendProgress;
}

/* FMessageRpcServer implementation
 *****************************************************************************/

void FMessageRpcServer::ProcessCancelation(const FMessageRpcCancel& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FReturnInfo ReturnInfo;

	if (Returns.RemoveAndCopyValue(Message.CallId, ReturnInfo))
	{
		ReturnInfo.Return->Cancel();
	}
}


void FMessageRpcServer::ProcessRequest(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	auto Message = (FRpcMessage*)Context->GetMessage();
	const FTopLevelAssetPath MessageType = Context->GetMessageTypePathName();

	if (!Handlers.Contains(MessageType))
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Keeping checking and executing NoHandlerDelegate for backwards compatibility 
		if (!NoHandlerDelegateWithPathName.IsBound() && !NoHandlerDelegate.IsBound())
		{
			return;
		}

		NoHandlerDelegateWithPathName.ExecuteIfBound(MessageType);
		NoHandlerDelegate.ExecuteIfBound(MessageType.GetAssetName());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	auto Handler = Handlers.FindRef(MessageType);

	if (Handler.IsValid())
	{
		FReturnInfo ReturnInfo;
		{
			ReturnInfo.ClientAddress = Context->GetSender();
			ReturnInfo.LastProgressSent = FDateTime::UtcNow();
			ReturnInfo.Return = Handler->HandleRequest(Context);
		}

		Returns.Add(Message->CallId, ReturnInfo);
	}
	else
	{
		// notify caller that call was not handled
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FMessageRpcUnhandled>(Message->CallId), Context->GetSender());
	}
}


void FMessageRpcServer::SendProgress(const FGuid& CallId, const FReturnInfo& ReturnInfo)
{
	const TSharedPtr<IAsyncProgress>& Progress = ReturnInfo.Progress;
	const TSharedPtr<IAsyncTask>& Task = ReturnInfo.Task;

	MessageEndpoint->Send(
		FMessageEndpoint::MakeMessage<FMessageRpcProgress>(
			CallId,
			Progress.IsValid() ? Progress->GetCompletion().Get(-1.0f) : -1.0f,
			Progress.IsValid() ? Progress->GetStatusText() : FText::GetEmpty()
		),
		ReturnInfo.ClientAddress
	);
}


void FMessageRpcServer::SendResult(const FGuid& CallId, const FReturnInfo& ReturnInfo)
{
	FRpcMessage* Message = ReturnInfo.Return->CreateResponseMessage();
	Message->CallId = CallId;

	MessageEndpoint->Send(
		Message,
		ReturnInfo.Return->GetResponseTypeInfo(),
		EMessageFlags::None,
		nullptr,
		TArrayBuilder<FMessageAddress>().Add(ReturnInfo.ClientAddress),
		FTimespan::Zero(),
		FDateTime::MaxValue()
	);
}


/* FMessageRpcServer event handlers
 *****************************************************************************/

void FMessageRpcServer::HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	const TWeakObjectPtr<UScriptStruct>& MessageTypeInfo = Context->GetMessageTypeInfo();

	if (!MessageTypeInfo.IsValid())
	{
		return;
	}

	if (MessageTypeInfo == FMessageRpcCancel::StaticStruct())
	{
		ProcessCancelation(*static_cast<const FMessageRpcCancel*>(Context->GetMessage()), Context);
	}
	else if (MessageTypeInfo->IsChildOf(FRpcMessage::StaticStruct()))
	{
		ProcessRequest(Context);
	}
}


bool FMessageRpcServer::HandleTicker(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageRpcServer_HandleTicker);

	const FDateTime UtcNow = FDateTime::UtcNow();

	for (TMap<FGuid, FReturnInfo>::TIterator It(Returns); It; ++It)
	{
		FReturnInfo& ReturnInfo = It.Value();

		if (ReturnInfo.Return->IsReady())
		{
			SendResult(It.Key(), ReturnInfo);
			It.RemoveCurrent();
		}
		else if (bSendProgress &&
			(UtcNow - ReturnInfo.LastProgressSent > FTimespan::FromSeconds(MESSAGE_RPC_PROGRESS_INTERVAL)))
		{
			SendProgress(It.Key(), ReturnInfo);
			ReturnInfo.LastProgressSent = UtcNow;
		}
	}

	return true;
}
