// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMessagingRpcModule.h"
#include "Modules/ModuleManager.h"
#include "MessageRpcClient.h"
#include "MessageRpcServer.h"


/**
 * Implements the MessagingRpc module.
 */
class FMessagingRpcModule
	: public IMessagingRpcModule
{
public:

	//~ IMessagingRpcModule interface

	virtual TSharedRef<IMessageRpcClient> CreateRpcClient() override
	{
		return MakeShared<FMessageRpcClient>();
	}

	virtual TSharedRef<IMessageRpcClient> CreateRpcClient(const FString& DebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& MessageBus) override
	{
		return MakeShared<FMessageRpcClient>(DebugName, MessageBus);
	}

	virtual TSharedRef<IMessageRpcServer> CreateRpcServer() override
	{
		return MakeShared<FMessageRpcServer>();
	}

	virtual TSharedRef<IMessageRpcServer> CreateRpcServer(const FString& DebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& MessageBus) override
	{
		return MakeShared<FMessageRpcServer>(DebugName, MessageBus);
	}
	
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};


IMPLEMENT_MODULE(FMessagingRpcModule, MessagingRpc);
