// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"

class FText;
class IMessageContext;
class UScriptStruct;

struct FGuid;


/**
 * Interface for RPC calls.
 *
 * Every time an RPC call is made, a request message containing the call parameters is
 * sent to the remote endpoint. While the remote endpoint is executing the call, it may
 * send back progress updates in regular intervals. Once the call is complete, the remote
 * endpoint sends a response message containing the result.
 */
class IMessageRpcCall
{
public:

	/**
	 * Complete the request and set its result, if available.
	 *
	 * @param ResponseContext The context of the response message.
	 * @see TimeOut
	 */
	virtual void Complete(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& ResponseContext) = 0;

	/**
	 * Get the call's unique identifier.
	 *
	 * @return Call identifier.
	 */
	virtual const FGuid& GetId() const = 0;

	/**
	 * Get the request message template.
	 *
	 * @return Pointer to the request message template.
	 */
	virtual void* GetMessageTemplate() const = 0;

	/**
	 * Constructs a new message based on the call message template. Ownership of the message belongs to the caller.
	 *
	 * @return Pointer to the request message.
	 */
	virtual void* ConstructMessage() const = 0;

	/**
	 * Get the time at which the request was last updated by the server.
	 *
	 * @return Update time.
	 */
	virtual FDateTime GetLastUpdated() const = 0;

	/**
	 * Get the type of the request message.
	 *
	 * @return Request message type.
	 */
	virtual UScriptStruct* GetMessageType() const = 0;

	/**
	 * Gets the time at which the request was created.
	 *
	 * @return Creation time.
	 */
	virtual FDateTime GetTimeCreated() const = 0;

	/**
	 * Time out the request.
	 *
	 * @see Complete
	 */
	virtual void TimeOut() = 0;

	/**
	 * Update the current progress.
	 *
	 * @param InCompletion The new completion percentage.
	 * @param InStatusText The new status text.
	 */
	virtual void UpdateProgress(float InCompletion, const FText& InStatusText) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageRpcCall() = default;
};
