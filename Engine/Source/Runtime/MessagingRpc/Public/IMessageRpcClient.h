// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MessageEndpoint.h"
#include "Misc/Guid.h"
#include "IMessageContext.h"
#include "IMessageRpcCall.h"
#include "Async/IAsyncProgress.h"
#include "Async/IAsyncTask.h"
#include "Async/Future.h"
#include "Async/AsyncResult.h"

/**
 * Interface for RPC clients.
 */
class IMessageRpcClient
{
	/** Abstract base class for RPC calls. */
	class FCall
		: public IAsyncProgress
		, public IAsyncTask
		, public IMessageRpcCall
	{
	public:

		FCall()
			: State(EAsyncTaskState::Running)
			, TimeCreated(FDateTime::UtcNow())
		{ }

	public:

		FSimpleDelegate& OnCancelled()
		{
			return CanceledDelegate;
		}

	public:

		virtual TOptional<float> GetCompletion() override
		{
			return Completion;
		}

		virtual FText GetStatusText() override
		{
			return StatusText;
		}

		virtual FSimpleDelegate& OnProgressChanged() override
		{
			return ProgressChangedDelegate;
		}

	public:

		virtual void Cancel() override
		{
			State = EAsyncTaskState::Cancelled;
			CanceledDelegate.ExecuteIfBound();
		}

		virtual EAsyncTaskState GetTaskState() override
		{
			return State;
		}

	public:

		virtual FDateTime GetLastUpdated() const override
		{
			return LastUpdated;
		}

		virtual FDateTime GetTimeCreated() const override
		{
			return TimeCreated;
		}

		virtual void UpdateProgress(float InCompletion, const FText& InStatusText) override
		{
			Completion = InCompletion;
			StatusText = InStatusText;
			LastUpdated = FDateTime::UtcNow();

			ProgressChangedDelegate.ExecuteIfBound();
		}

	protected:

		float Completion;
		FDateTime LastUpdated;
		EAsyncTaskState State;
		FText StatusText;
		FDateTime TimeCreated;

	private:

		FSimpleDelegate CanceledDelegate;
		FSimpleDelegate ProgressChangedDelegate;
	};


	/** Template for RPC requests. */
	template<typename RpcType, typename... P>
	class TCall
		: public FCall
	{
	public:

		TCall(const FGuid& CallId, P... Params)
			: MessageTemplate(new typename RpcType::FRequest(Params...))
		{
			MessageTemplate->CallId = CallId;
		}

		virtual ~TCall()
		{
			delete MessageTemplate;
		}

		virtual void Complete(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& ResponseContext) override
		{
			if (ResponseContext->GetMessageTypeInfo() == RpcType::FResponse::StaticStruct())
			{
				State = EAsyncTaskState::Completed;
				Promise.SetValue(static_cast<const typename RpcType::FResponse*>(ResponseContext->GetMessage())->Result);
			}
			else
			{
				State = EAsyncTaskState::Failed;
				Promise.SetValue(typename RpcType::FResult());
			}
		}

		virtual void* ConstructMessage() const override
		{
			return FMessageEndpoint::MakeMessage<typename RpcType::FRequest>(*MessageTemplate);
		}

		TFuture<typename RpcType::FResult> GetFuture()
		{
			return Promise.GetFuture();
		}

		virtual const FGuid& GetId() const override
		{
			return MessageTemplate->CallId;
		}

		virtual void* GetMessageTemplate() const override
		{
			return MessageTemplate;
		}

		virtual UScriptStruct* GetMessageType() const override
		{
			return RpcType::FRequest::StaticStruct();
		}

		virtual void TimeOut() override
		{
			State = EAsyncTaskState::Failed;
			Promise.SetValue(typename RpcType::FResult());
		}

	private:

		TPromise<typename RpcType::FResult> Promise;
		typename RpcType::FRequest* MessageTemplate;
	};

public:

	/**
	 * Connect this client to an RPC server.
	 *
	 * @param InServerAddress The RPC server's message address.
	 * @see Disconnect
	 */
	virtual void Connect(const FMessageAddress& InServerAddress) = 0;

	/**
	 * Disconnect this client from the RPC server.
	 *
	 * @see Connect
	 */
	virtual void Disconnect() = 0;

	/**
	 * Whether the MessageRpcClient is currently connected to an RPC server.
	 *
	 * @return true if the client is connected, false otherwise.
	 */
	virtual bool IsConnected() const = 0;

public:

	/**
	 * Call a remote procedure.
	 *
	 * @param RpcType The RPC type definition.
	 * @param P The call parameter types.
	 * @param Params The call parameter list.
	 */
	template<typename RpcType, typename... P>
	TAsyncResult<typename RpcType::FResult> Call(P... Params)
	{
		typedef TCall<RpcType, P...> CallType;

		FGuid CallId = FGuid::NewGuid();
		TSharedRef<CallType> Call = MakeShareable(new CallType(CallId, Params...));
		{
			AddCall(Call);

			Call->OnCancelled().BindLambda([this, CallId]() {
				CancelCall(CallId);
			});	
		}

		return TAsyncResult<typename RpcType::FResult>(Call->GetFuture(), Call, Call);
	}

protected:

	/**
	 * Add an RPC request.
	 *
	 * @param Request The call to add.
	 */
	virtual void AddCall(const TSharedRef<IMessageRpcCall>& Call) = 0;

	/**
	 * Cancel the specified RPC call.
	 *
	 * @param CallId The unique identifier of the call to cancel.
	 */
	virtual void CancelCall(const FGuid& CallId) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageRpcClient() = default;
};


Expose_TNameOf(IMessageRpcClient)
