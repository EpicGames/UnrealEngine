// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMessageContext;
class IMessageRpcReturn;


/**
 * Interface for RPC request handlers.
 */
class IMessageRpcHandler
{
public:

	/**
	 * Handle an RPC request.
	 *
	 * @param Context The context of the RPC request message.
	 * @return An object for the pending RPC return value.
	 */
	virtual TSharedRef<IMessageRpcReturn> HandleRequest(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageRpcHandler() = default;
};
