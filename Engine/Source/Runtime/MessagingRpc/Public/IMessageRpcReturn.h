// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UScriptStruct;
struct FRpcMessage;


/**
 * Interface for RPC return values.
 */
class IMessageRpcReturn
{
public:

	/** Cancel the computation of the return value. */
	virtual void Cancel() = 0;

	/**
	 */
	virtual FRpcMessage* CreateResponseMessage() const = 0;

	/**
	 * Gets the type information for the corresponding RPC response message.
	 *
	 * @return The message type information.
	 */
	virtual UScriptStruct* GetResponseTypeInfo() const = 0;

	/**
	 * Check whether the return value is ready.
	 *
	 * @return true if the return value is ready, false otherwise.
	 */
	virtual bool IsReady() const = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageRpcReturn() = default;
};
