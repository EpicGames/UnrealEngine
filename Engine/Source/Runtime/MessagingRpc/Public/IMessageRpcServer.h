// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "Async/IAsyncTask.h"
#include "Async/AsyncResult.h"
#include "IMessageRpcHandler.h"
#include "IMessageRpcReturn.h"

struct FRpcMessage;

template<typename ResultType> class TFuture;

/** Delegate type for RPC messages that have no registered handler. */
UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use FOnMessagePathNameRpcNoHandler.")
DECLARE_DELEGATE_OneParam(FOnMessageRpcNoHandler, const FName& /*MessageType*/)

DECLARE_DELEGATE_OneParam(FOnMessagePathNameRpcNoHandler, const FTopLevelAssetPath& /*MessageType*/)

/**
 * Interface for RPC servers.
 */
class IMessageRpcServer
{
	/** Template for RPC results. */
	template<typename RpcType>
	class TReturn
		: public IMessageRpcReturn
	{
	public:

		TReturn(TAsyncResult<typename RpcType::FResult>&& InResult)
			: Result(MoveTemp(InResult))
		{ }

		virtual void Cancel() override
		{
			auto Task = Result.GetTask();

			if (Task.IsValid())
			{
				Task->Cancel();
			}
		}

		virtual FRpcMessage* CreateResponseMessage() const override
		{
			const TFuture<typename RpcType::FResult>& Future = Result.GetFuture();
			check(Future.IsReady());

			return FMessageEndpoint::MakeMessage<typename RpcType::FResponse>(Future.Get());
		}

		virtual UScriptStruct* GetResponseTypeInfo() const override
		{
			return RpcType::FResponse::StaticStruct();
		}

		virtual bool IsReady() const override
		{
			return Result.GetFuture().IsReady();
		}

		TAsyncResult<typename RpcType::FResult> Result;
	};


	/** Template for RPC request handlers. */
	template<typename RpcType, typename HandlerType>
	class THandler
		: public IMessageRpcHandler
	{
	public:

		typedef TAsyncResult<typename RpcType::FResult>(HandlerType::*FuncType)(const typename RpcType::FRequest&);

		THandler( HandlerType* InHandler, FuncType InFunc )
			: Handler(InHandler)
			, Func(InFunc)
		{
			check(InHandler != nullptr);
		}

		virtual TSharedRef<IMessageRpcReturn> HandleRequest(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
		{
			return MakeShareable(new TReturn<RpcType>((Handler->*Func)(*static_cast<const typename RpcType::FRequest*>(Context->GetMessage()))));
		}
	
	private:

		HandlerType* Handler;
		FuncType Func;
	};

public:

	/**
	 * Add an RPC request handler.
	 *
	 * @param Handler The handler to add.
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	inline void AddHandler(const FName& RequestMessageType, const TSharedRef<IMessageRpcHandler>& Handler)
	{
		AddHandler(UClass::TryConvertShortTypeNameToPathName<UStruct>(RequestMessageType.ToString()), Handler);
	}

	/**
	 * Add an RPC request handler.
	 *
	 * @param Handler The handler to add.
	 */
	virtual void AddHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IMessageRpcHandler>& Handler) = 0;

	/**
	 * Gets the server's message address.
	 *
	 * @return Message address.
	 */
	virtual const FMessageAddress& GetAddress() const = 0;

	/**
	 * Get a delegate that is executed when a received RPC message has no registered handler.
	 *
	 * @return The delegate.
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use OnNoHandlerWithPathName.")	
	virtual FOnMessageRpcNoHandler& OnNoHandler() = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Get a delegate that is executed when a received RPC message has no registered handler.
	 *
	 * @return The delegate.
	 */
	virtual FOnMessagePathNameRpcNoHandler& OnNoHandlerWithPathName() = 0;

	/** 
	 * Set if the server sends progress updates
	 * @param InSendProgress if the server send rpc progress update
	 */
	virtual void SetSendProgressUpdate(bool InSendProgress) = 0;

public:

	/**
	 * Register an RPC request handler.
	 *
	 * @param Handler The object that will handle the requests.
	 * @param HandlerFunc The object's request handling function.
	 */
	template<typename RpcType, typename HandlerType>
	void RegisterHandler(HandlerType* Handler, typename THandler<RpcType, HandlerType>::FuncType Func)
	{
		AddHandler(RpcType::FRequest::StaticStruct()->GetFName(), MakeShareable(new THandler<RpcType, HandlerType>(Handler, Func)));
	}

public:

	/** Virtual destructor. */
	virtual ~IMessageRpcServer() = default;
};
