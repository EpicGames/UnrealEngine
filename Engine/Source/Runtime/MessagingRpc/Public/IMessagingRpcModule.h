// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IMessageBus;
class IMessageRpcClient;
class IMessageRpcServer;

struct FMessageEndpointBuilder;


/**
 * Interface for the MessagingRpc module.
 */
class IMessagingRpcModule
	: public IModuleInterface
{
public:

	/**
	 * Create a client for remote procedure calls.
	 *
	 * @return The RPC client.
	 */
	virtual TSharedRef<IMessageRpcClient> CreateRpcClient() = 0;
	
	/**
	 * Create a client for remote procedure calls.
	 *
	 * @param DebugName The RPC client's endpoint debug name.
	 * @param MessageBus The RPC client's endpoint message bus.
	 * @return The RPC client.
	 */
	virtual TSharedRef<IMessageRpcClient> CreateRpcClient(const FString& DebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& MessageBus) = 0;
	
	/**
	 * Create a server for remote procedure calls.
	 *
	 * @return The RPC server.
	 */
	virtual TSharedRef<IMessageRpcServer> CreateRpcServer() = 0;

	/**
	 * Create a server for remote procedure calls.
	 *
	 * @param DebugName The RPC server's endpoint debug name.
	 * @param MessageBus The RPC server's endpoint message bus.
	 * @return The RPC server.
	 */
	virtual TSharedRef<IMessageRpcServer> CreateRpcServer(const FString& DebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& MessageBus) = 0;
	
public:

	/** Virtual destructor. */
	virtual ~IMessagingRpcModule() = default;
};
