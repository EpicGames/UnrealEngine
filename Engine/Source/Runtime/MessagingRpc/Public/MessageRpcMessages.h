// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"

#include "MessageRpcMessages.generated.h"


/** Message for canceling an RPC call. */
USTRUCT()
struct FMessageRpcCancel
{
	GENERATED_USTRUCT_BODY()

	/** Correlation identifier for the RPC call that this message refers to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid CallId;

	/** Default constructor. */
	FMessageRpcCancel() { }

	/** Creates and initializes a new instance. */
	FMessageRpcCancel(const FGuid& InCallId)
		: CallId(InCallId)
	{ }
};


/** Message for updating the progress of an RPC call. */
USTRUCT()
struct FMessageRpcProgress
{
	GENERATED_USTRUCT_BODY()

	/** Completion percentage (0.0 to 1.0). */
	UPROPERTY(EditAnywhere, Category="Message")
	float Completion;

	/** Correlation identifier for the RPC call that this message refers to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid CallId;

	/** Status text. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString StatusText;

	/** Default constructor. */
	FMessageRpcProgress() : Completion(0.0f) { }

	/** Creates and initializes a new instance. */
	FMessageRpcProgress(const FGuid& InCallId, float InCompletion, const FText& InStatusText)
		: Completion(InCompletion)
		, CallId(InCallId)
		, StatusText(InStatusText.ToString())
	{ }
};


/** Message for notifying RPC clients that a call was not handled by the server. */
USTRUCT()
struct FMessageRpcUnhandled
{
	GENERATED_USTRUCT_BODY()

	/** Correlation identifier for the RPC call that this message refers to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid CallId;

	/** Default constructor. */
	FMessageRpcUnhandled() { }

	/** Creates and initializes a new instance. */
	FMessageRpcUnhandled(const FGuid& InCallId)
		: CallId(InCallId)
	{ }
};
