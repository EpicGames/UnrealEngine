// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "IMessageContext.h"
#include "IMessageRpcServer.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/TopLevelAssetPath.h"

class FMessageEndpoint;
class FMessagingRpcModule;
class IAsyncProgress;
class IAsyncTask;
class IMessageBus;
class IMessageRpcHandler;
class IMessageRpcReturn;
struct FMessageEndpointBuilder;
struct FMessageRpcCancel;


/**
 * Implements an RPC server.
 */
class FMessageRpcServer
	: public IMessageRpcServer
{
public:
	/** Default constructor. */
	MESSAGINGRPC_API FMessageRpcServer();
	MESSAGINGRPC_API FMessageRpcServer(const FString& InDebugName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);

	/** Virtual destructor. */
	MESSAGINGRPC_API virtual ~FMessageRpcServer();

public:

	//~ IMessageRpcServer interface

	MESSAGINGRPC_API virtual void AddHandler(const FTopLevelAssetPath& RequestMessageType, const TSharedRef<IMessageRpcHandler>& Handler) override;
	MESSAGINGRPC_API virtual const FMessageAddress& GetAddress() const override;
MESSAGINGRPC_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FOnMessageRpcNoHandler& OnNoHandler() override;
MESSAGINGRPC_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FOnMessagePathNameRpcNoHandler& OnNoHandlerWithPathName() override;
	MESSAGINGRPC_API virtual void SetSendProgressUpdate(bool InSendProgress) override;
protected:
	MESSAGINGRPC_API explicit FMessageRpcServer(FMessageEndpointBuilder&& InEndpointBuilder);

	struct FReturnInfo
	{
		FMessageAddress ClientAddress;
		FDateTime LastProgressSent;
		TSharedPtr<IAsyncProgress> Progress;
		TSharedPtr<IMessageRpcReturn> Return;
		TSharedPtr<IAsyncTask> Task;
	};

	MESSAGINGRPC_API TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetEndpoint() const;

	/** Processes an FMessageRpcCancel message. */
	MESSAGINGRPC_API void ProcessCancelation(const FMessageRpcCancel& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Processes an RPC request message. */
	MESSAGINGRPC_API void ProcessRequest(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Send a progress message to the RPC client that made the RPC call. */
	MESSAGINGRPC_API void SendProgress(const FGuid& CallId, const FReturnInfo& ReturnInfo);

	/** Send a result message to the RPC client that made the RPC call. */
	MESSAGINGRPC_API void SendResult(const FGuid& CallId, const FReturnInfo& ReturnInfo);

protected:

	/** Message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

private:

	/** Handles all incoming messages. */
	void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles the ticker. */
	bool HandleTicker(float DeltaTime);

private:

	/** Registered request message handlers. */
	TMap<FTopLevelAssetPath, TSharedPtr<IMessageRpcHandler>> Handlers;

	/* Delegate that is executed when a received RPC message has no registered handler. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnMessageRpcNoHandler NoHandlerDelegate;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/* Delegate that is executed when a received RPC message has no registered handler. */
	FOnMessagePathNameRpcNoHandler NoHandlerDelegateWithPathName;

	/** Collection of pending RPC returns. */
	TMap<FGuid, FReturnInfo> Returns;

	/** Handle to the registered ticker. */
	FTSTicker::FDelegateHandle TickerHandle;

	/** If the server sends progress updates. */
	bool bSendProgress = true;
};
