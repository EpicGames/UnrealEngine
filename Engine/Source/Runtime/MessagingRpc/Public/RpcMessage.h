// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"

#include "RpcMessage.generated.h"


/**
 * Base type for RPC messages.
 */
USTRUCT()
struct FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	/** Correlation identifier for the RPC call that this message refers to. */
	UPROPERTY()
	FGuid CallId;

	/** Default constructor. */
	FRpcMessage() { }
};


#define DECLARE_RPC(RpcType, ResultType) \
	struct RpcType \
	{ \
		typedef RpcType##Request FRequest; \
		typedef RpcType##Response FResponse; \
		typedef ResultType FResult; \
	};
